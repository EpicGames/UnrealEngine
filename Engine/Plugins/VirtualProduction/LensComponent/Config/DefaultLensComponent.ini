[CoreRedirects]

;
; 5.4 Redirects
;

; Classes

+ClassRedirects=(OldName="/Script/CameraCalibrationCore.LensComponent", NewName="/Script/LensComponent.LensComponent")
+ClassRedirects=(OldName="/Script/CameraCalibrationCoreMovieScene.MovieSceneLensComponentSection", NewName="/Script/LensComponentEditor.MovieSceneLensComponentSection")
+ClassRedirects=(OldName="/Script/CameraCalibrationCoreMovieScene.MovieSceneLensComponentTrack", NewName="/Script/LensComponentEditor.MovieSceneLensComponentTrack")
+ClassRedirects=(OldName="/Script/CameraCalibrationCoreSequencer.MovieSceneLensComponentTrackRecorder", NewName="/Script/LensComponentEditor.MovieSceneLensComponentTrackRecorder")

;
; 5.7 Redirects
;

; Enums

+EnumRedirects=(OldName="/Script/LensComponent.EDistortionRenderingMode", NewName="/Script/CameraCalibrationCore.EDistortionRenderingMode")
