{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Lens Component",
	"Description": "Implements the Lens Component for adding distortion to a cinematic camera",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Hidden": true,
	"Installed": false,
	"SupportedPrograms": [ "LiveLinkHub" ],
	"Modules": [
		{
			"Name": "LensComponent",
			"Type": "Runtime",
			"ProgramAllowlist": [ "LiveLinkHub" ],
			"LoadingPhase": "Default"
		},
		{
			"Name": "LensComponentEditor",
			"Type": "Editor",
			"ProgramAllowlist": [ "LiveLinkHub" ],
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "CameraCalibrationCore",
			"Enabled": true
		},
		{
			"Name": "LiveLink",
			"Enabled": true
		},
		{
			"Name": "Takes",
			"Enabled": true
		}
	]
}