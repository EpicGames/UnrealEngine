// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LensComponent : ModuleRules
	{
		public LensComponent(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"LiveLinkComponents",
					"LiveLinkInterface",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CameraCalibrationCore",
					"CinematicCamera",
					"Core",
					"CoreUObject",
					"Engine",
				}
			);
		}
	}
}
