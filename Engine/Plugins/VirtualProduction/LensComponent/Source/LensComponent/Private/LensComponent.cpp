// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensComponent.h"

#include "Camera/CameraActor.h"
#include "CameraCalibrationSettings.h"
#include "CameraCalibrationSubsystem.h"
#include "CineCameraComponent.h"
#include "Controllers/LiveLinkTransformController.h"
#include "Engine/Engine.h"
#include "ILiveLinkComponentModule.h"
#include "Modules/ModuleManager.h"
#include "Roles/LiveLinkCameraTypes.h"
#include "Roles/LiveLinkCameraRole.h"
#include "Roles/LiveLinkTransformRole.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LensComponent)

DEFINE_LOG_CATEGORY_STATIC(LogLensComponent, Log, All);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
ULensComponent::ULensComponent()
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	// This component is designed to tick after the LiveLink component, which uses TG_PrePhysics
	// We also use a tick prerequisite on LiveLink components, so technically this could also use TG_PrePhysics
	PrimaryComponentTick.TickGroup = ETickingGroup::TG_DuringPhysics;
	bTickInEditor = true;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void ULensComponent::OnRegister()
{
	Super::OnRegister();

	InitDefaultCamera();

	// The component might already have a serialized LensModel, but might not go through PostLoad (in the case that this is spawned by sequencer)
	// so attempt to create a distortion handler now for the currently set LensModel. 
	CreateDistortionHandler();

	// Register a callback to let us know when a new LiveLinkComponent is added to this component's parent actor
	// This gives us the opportunity to track when the subject representation of that LiveLinkComponent changes
	ILiveLinkComponentsModule& LiveLinkComponentsModule = FModuleManager::GetModuleChecked<ILiveLinkComponentsModule>(TEXT("LiveLinkComponents"));
	if (!LiveLinkComponentsModule.OnLiveLinkComponentRegistered().IsBoundToObject(this))
	{
		LiveLinkComponentsModule.OnLiveLinkComponentRegistered().AddUObject(this, &ULensComponent::OnLiveLinkComponentRegistered);
	}

	// Look for any LiveLinkComponents that were previously added to this component's parent actor
	TInlineComponentArray<ULiveLinkComponentController*> LiveLinkComponents;
	if (AActor* Owner = GetOwner())
	{
		Owner->GetComponents(LiveLinkComponents);
	}

	for (ULiveLinkComponentController* LiveLinkComponent : LiveLinkComponents)
	{	
		AddTickPrerequisiteComponent(LiveLinkComponent);

		if (!LiveLinkComponent->OnLiveLinkControllersTicked().IsBoundToObject(this))
		{
			LiveLinkComponent->OnLiveLinkControllersTicked().AddUObject(this, &ULensComponent::ProcessLiveLinkData);
		}
	}
}

void ULensComponent::OnUnregister()
{
	Super::OnUnregister();

	ILiveLinkComponentsModule& LiveLinkComponentsModule = FModuleManager::GetModuleChecked<ILiveLinkComponentsModule>(TEXT("LiveLinkComponents"));
	LiveLinkComponentsModule.OnLiveLinkComponentRegistered().RemoveAll(this);

	TInlineComponentArray<ULiveLinkComponentController*> LiveLinkComponents;
	if (AActor* Owner = GetOwner())
	{
		Owner->GetComponents(LiveLinkComponents);
	}

	for (ULiveLinkComponentController* LiveLinkComponent : LiveLinkComponents)
	{
		LiveLinkComponent->OnLiveLinkControllersTicked().RemoveAll(this);
	}
}

void ULensComponent::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	if (UCineCameraComponent* const CineCameraComponent = Cast<UCineCameraComponent>(TargetCameraComponent.GetComponent(GetOwner())))
	{
		// If the camera's focal length has changed since the last time this component ticked, we need to update the original focal length we are using
		if (CineCameraComponent->CurrentFocalLength != LastFocalLength)
		{
			// If this is a recorded lens component, then there are no external forces changing the focal length. Original focal length should be preserved.
			if (EvaluationMode != EFIZEvaluationMode::UseRecordedValues)
			{
				OriginalFocalLength = CineCameraComponent->CurrentFocalLength;
			}
		}

		// Get the focus and zoom values needed to evaluate the LensFile this Tick
		UpdateLensFileEvaluationInputs(CineCameraComponent);

		// Attempt to apply nodal offset, which will only succeed if there is a valid LensFile, evaluation inputs, and component to offset
		if (bApplyNodalOffsetOnTick)
		{
			ApplyNodalOffset();
		}

		ULensFile* LensFile = LensFilePicker.GetLensFile();

		UpdateCameraFilmback(CineCameraComponent);

		EvaluateFocalLength(CineCameraComponent);

		// Evaluate Distortion
		bWasDistortionEvaluated = false;
		TObjectPtr<ULensDistortionModelHandlerBase> LensDistortionHandler = LensDistortionHandlerMap.FindRef(LensModel);

		FDisplacementMapBlendingParams BlendState;

		if (LensDistortionHandler)
		{
			switch (DistortionStateSource)
			{
			case EDistortionSource::LensFile:
			{
				if (!LensFile || !EvalInputs.bIsValid)
				{
					break;
				}

				LensFile->EvaluateDistortionData(EvalInputs.Focus, EvalInputs.Zoom, FVector2D(EvalInputs.Filmback.SensorWidth, EvalInputs.Filmback.SensorHeight), LensDistortionHandler);

				DistortionState = LensDistortionHandler->GetCurrentDistortionState();

				LensFile->GetBlendState(EvalInputs.Focus, EvalInputs.Zoom, FVector2D(EvalInputs.Filmback.SensorWidth, EvalInputs.Filmback.SensorHeight), BlendState);

				// Adjust overscan by the overscan multiplier
				if (bScaleOverscan)
				{
					const float ScaledOverscanFactor = ((LensDistortionHandler->GetOverscanFactor() - 1.0f) * OverscanMultiplier) + 1.0f;
					LensDistortionHandler->SetOverscanFactor(ScaledOverscanFactor);
				}

				bWasDistortionEvaluated = true;
				break;
			}
			case EDistortionSource::LiveLinkLensSubject:
			{
				// This case is used by the customization to make the distortion state properties read-only. Otherwise, behavior is the same as EDistortionSource::Manual.
				// falls through
			}
			case EDistortionSource::Manual:
			{
				LensDistortionHandler->SetDistortionState(DistortionState);
				LensDistortionHandler->SetCameraFilmback(CineCameraComponent->Filmback);

				BlendState.States[0] = DistortionState;
				BlendState.BlendType = EDisplacementMapBlendType::OneFocusOneZoom;

				//Recompute overscan factor for the distortion state
				float OverscanFactor = LensDistortionHandler->ComputeOverscanFactor();

				if (bScaleOverscan)
				{
					OverscanFactor = ((OverscanFactor - 1.0f) * OverscanMultiplier) + 1.0f;
				}
				LensDistortionHandler->SetOverscanFactor(OverscanFactor);

				//Make sure the displacement map is up to date
				LensDistortionHandler->ProcessCurrentDistortion();

				bWasDistortionEvaluated = true;
				break;
			}
			default:
			{
				ensureMsgf(false, TEXT("Unhandled EDistortionSource type. Update this switch statement."));
				break; // Do nothing
			}
			}
		}

		if (bApplyDistortion)
		{
			if (LensDistortionHandler && bWasDistortionEvaluated)
			{
				const EDistortionRenderingMode ResolvedRenderingMode = GetDistortionRenderingMode();
				
				// Check if the resolved rendering mode has changed (can happen with Preferred mode)
				if (bIsDistortionSetup && ResolvedRenderingMode != LastAppliedRenderingMode)
				{
					// Clean up the previous mode before switching
					CleanupDistortion(CineCameraComponent);
				}
				
				if (ResolvedRenderingMode == EDistortionRenderingMode::PostProcessMaterial)
				{
					// Get the current distortion MID from the lens distortion handler
					UMaterialInstanceDynamic* NewDistortionMID = LensDistortionHandler->GetDistortionMID();

					// If the MID has changed
					if (LastDistortionMID != NewDistortionMID)
					{
						CineCameraComponent->RemoveBlendable(LastDistortionMID);
						CineCameraComponent->AddOrUpdateBlendable(NewDistortionMID);
					}

					// Cache the latest distortion MID
					LastDistortionMID = NewDistortionMID;

					bIsDistortionSetup = true;
					LastAppliedRenderingMode = ResolvedRenderingMode;
				}
				else if (ResolvedRenderingMode == EDistortionRenderingMode::TemporalSuperResolution || ResolvedRenderingMode == EDistortionRenderingMode::PassAfterTonemap)
				{
					if (UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>())
					{
						if (ACameraActor* CameraActor = Cast<ACameraActor>(GetOwner()))
						{
							SubSystem->SetLensDistortionSVEState(CameraActor, BlendState, LensDistortionHandler, ResolvedRenderingMode);
						}
					}

					bIsDistortionSetup = true;
					LastAppliedRenderingMode = ResolvedRenderingMode;
				}

				// Set the camera's overscan settings
				if (bOverrideCameraOverscan)
				{
					const float DesiredOverscan = FMath::Clamp(LensDistortionHandler->GetOverscanFactor() - 1.0f, 0.0f, 1.0f);
					CineCameraComponent->Overscan = DesiredOverscan;

					// The overscan factor used by the post process material needs to be equivalent to what is set on the camera in order for the PPM and SVE paths to render the same
					LensDistortionHandler->SetOverscanFactor(CineCameraComponent->Overscan + 1.0f);

 					CineCameraComponent->bCropOverscan = (ResolvedRenderingMode == EDistortionRenderingMode::TemporalSuperResolution);
				}
			}
			else
			{
				CleanupDistortion(CineCameraComponent);
			}
		}
		else
		{
			// If this is a recorded lens component, and distortion is disabled, set the focal length the original focal length to override the recorded value from the camera.
			if (EvaluationMode == EFIZEvaluationMode::UseRecordedValues)
			{
				CineCameraComponent->CurrentFocalLength = OriginalFocalLength;
			}

			CleanupDistortion(CineCameraComponent);
		}

		LastFocalLength = CineCameraComponent->CurrentFocalLength;
	}

	bWasLiveLinkFIZUpdated = false;

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
}

void ULensComponent::DestroyComponent(bool bPromoteChildren /*= false*/)
{
	Super::DestroyComponent(bPromoteChildren);

	if (UCineCameraComponent* const CineCameraComponent = Cast<UCineCameraComponent>(TargetCameraComponent.GetComponent(GetOwner())))
	{
		CleanupDistortion(CineCameraComponent);
	}
}

void ULensComponent::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	InitDefaultCamera();
}

void ULensComponent::PostEditImport()
{
	Super::PostEditImport();

	InitDefaultCamera();
}

void ULensComponent::PostLoad()
{
	Super::PostLoad();

	// After LensModel has been loaded, create a new handler
	CreateDistortionHandler();

	if (bIsDistortionSetup)
	{
		if (UCineCameraComponent* const CineCameraComponent = Cast<UCineCameraComponent>(TargetCameraComponent.GetComponent(GetOwner())))
		{
			CleanupDistortion(CineCameraComponent);
		}
		bIsDistortionSetup = false;
	}

	// Cache the currently set LensFile and register for new LensFile events
	WeakCachedLensFile = LensFilePicker.GetLensFile();

	if (ULensFile* const LensFile = WeakCachedLensFile.Get())
	{
		if (!LensFile->OnLensFileModelChanged().IsBoundToObject(this))
		{
			LensFile->OnLensFileModelChanged().AddUObject(this, &ULensComponent::OnLensFileModelChanged);
		}
	}

#if WITH_EDITOR
	const int32 UE5MainVersion = GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID);
	if (UE5MainVersion < FUE5MainStreamObjectVersion::LensComponentNodalOffset)
	{
		// If this component was previously applying nodal offset, reset the camera component's transform to its original pose
		if (UCineCameraComponent* const CineCameraComponent = Cast<UCineCameraComponent>(TargetCameraComponent.GetComponent(GetOwner())))
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (bApplyNodalOffsetOnTick)
			{
				CineCameraComponent->SetRelativeLocation(OriginalCameraLocation_DEPRECATED);
				CineCameraComponent->SetRelativeRotation(OriginalCameraRotation_DEPRECATED.Quaternion());
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif //WITH_EDITOR
}

void ULensComponent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		// For backwards compatibility, preserve the old default value
		if ((Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LensComponentDefaultToDistortionSVE)
			&& !GetArchetype()->IsInBlueprint())
		{
			DistortionRenderingMode = EDistortionRenderingMode::PostProcessMaterial;
		}
	}
#endif

	Super::Serialize(Ar);
}

#if WITH_EDITOR
void ULensComponent::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ULensComponent, bApplyDistortion))
	{
		SetApplyDistortion(bApplyDistortion);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ULensComponent, DistortionRenderingMode))
	{
		if (UCineCameraComponent* const CineCameraComponent = Cast<UCineCameraComponent>(TargetCameraComponent.GetComponent(GetOwner())))
		{
			CleanupDistortion(CineCameraComponent);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ULensComponent, TargetCameraComponent))
	{
		// Clean up distortion on the last target camera
		if (LastCameraComponent)
		{
			CleanupDistortion(LastCameraComponent);
		}

		LastCameraComponent = Cast<UCineCameraComponent>(TargetCameraComponent.GetComponent(GetOwner()));
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(FLensFilePicker, LensFile))
	{
		SetLensFilePicker(LensFilePicker);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ULensComponent, LensModel))
	{
		SetLensModel(LensModel);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ULensComponent, DistortionStateSource))
	{
		SetDistortionSource(DistortionStateSource);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(ULensComponent, bOverrideCameraOverscan))
	{
		SetOverrideCameraOverscan(bOverrideCameraOverscan);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif //WITH_EDITOR

void ULensComponent::ReapplyNodalOffset()
{
	// Find the TrackedComponent based on the serialized name
	TInlineComponentArray<USceneComponent*> SceneComponents;
	if (AActor* Owner = GetOwner())
	{
		Owner->GetComponents(SceneComponents);
	}

	for (USceneComponent* SceneComponent : SceneComponents)
	{
		if (SceneComponent->GetName() == TrackedComponentName)
		{
			TrackedComponent = SceneComponent;
			break;
		}
	}

	if (TrackedComponent.IsValid())
	{
		// Reset the tracked component back to its original relative transform (before nodal offset was originally applied)
		TrackedComponent->SetRelativeTransform(GetOriginalTrackedComponentTransform());

		// Now, reapply the nodal offset to the tracked component
		ApplyNodalOffset();
	}
}

void ULensComponent::ApplyNodalOffset()
{
	// Verify that we detected a tracked component for us to offset this tick. If there is none, it is likely because no LiveLink transform controller executed this tick.
	if (!TrackedComponent.IsValid())
	{
		return;
	}

	// Verify that the evaluation inputs are valid
	if ((EvaluationMode == EFIZEvaluationMode::DoNotEvaluate) || !EvalInputs.bIsValid) 
	{
		return;
	}

	// Verify that there is a valid LensFile to evaluate
	ULensFile* LensFile = LensFilePicker.GetLensFile();
	if (!LensFile)
	{
		return;
	}

	FNodalPointOffset Offset;
	LensFile->EvaluateNodalPointOffset(EvalInputs.Focus, EvalInputs.Zoom, Offset);

	// Cache the original transform before applying the offset, so that nodal offset can potentially be re-evaluated in the future
	SetOriginalTrackedComponentTransform(TrackedComponent.Get()->GetRelativeTransform());

	TrackedComponent.Get()->AddLocalOffset(Offset.LocationOffset);
	TrackedComponent.Get()->AddLocalRotation(Offset.RotationOffset);

	bWasNodalOffsetAppliedThisTick = true;

	// Reset so that nodal offset will only be applied again next tick if new tracking data was applied between now and then
	TrackedComponent.Reset();
}

void ULensComponent::ApplyNodalOffset(USceneComponent* ComponentToOffset, bool bUseManualInputs, float ManualFocusInput, float ManualZoomInput)
{
	bApplyNodalOffsetOnTick = false;

	// Verify that the input component was not null
	if (!ComponentToOffset)
	{
		return;
	}

	// Verify that there is a valid LensFile to evaluate
	ULensFile* LensFile = LensFilePicker.GetLensFile();
	if (!LensFile)
	{
		return;
	}

	FNodalPointOffset Offset;
	if (bUseManualInputs)
	{
		LensFile->EvaluateNodalPointOffset(ManualFocusInput, ManualZoomInput, Offset);
	}
	else
	{
		if (!EvalInputs.bIsValid)
		{
			return;
		}

		LensFile->EvaluateNodalPointOffset(EvalInputs.Focus, EvalInputs.Zoom, Offset);
	}

	// Cache the original transform before applying the offset, so that nodal offset can potentially be re-evaluated in the future
	SetOriginalTrackedComponentTransform(ComponentToOffset->GetRelativeTransform());

	ComponentToOffset->AddLocalOffset(Offset.LocationOffset);
	ComponentToOffset->AddLocalRotation(Offset.RotationOffset);
}

FTransform ULensComponent::GetOriginalTrackedComponentTransform()
{
	OriginalTrackedComponentTransform.SetLocation(OriginalTrackedComponentLocation);

	FRotator TrackedRotator;
	TrackedRotator.Pitch = OriginalTrackedComponentRotation.Y;
	TrackedRotator.Yaw = OriginalTrackedComponentRotation.Z;
	TrackedRotator.Roll = OriginalTrackedComponentRotation.X;

	OriginalTrackedComponentTransform.SetRotation(TrackedRotator.Quaternion());
	return OriginalTrackedComponentTransform;
}

void ULensComponent::SetOriginalTrackedComponentTransform(const FTransform& NewTransform)
{
	OriginalTrackedComponentTransform = NewTransform;
	OriginalTrackedComponentLocation = NewTransform.GetLocation();

	FRotator NewRotator = NewTransform.GetRotation().Rotator();
	OriginalTrackedComponentRotation.X = NewRotator.Roll;
	OriginalTrackedComponentRotation.Y = NewRotator.Pitch;
	OriginalTrackedComponentRotation.Z = NewRotator.Yaw;
}

void ULensComponent::RestoreCameraOverscanDefaults()
{
	const UCineCameraComponent* CDO = UCineCameraComponent::StaticClass()->GetDefaultObject<UCineCameraComponent>();
	UCineCameraComponent* CineCameraComponent = Cast<UCineCameraComponent>(TargetCameraComponent.GetComponent(GetOwner()));

	if (IsValid(CineCameraComponent))
	{
		CineCameraComponent->Modify();
		CineCameraComponent->Overscan = CDO->Overscan;
		CineCameraComponent->bCropOverscan = CDO->bCropOverscan;
	}
}

void ULensComponent::EvaluateFocalLength(UCineCameraComponent* CineCameraComponent)
{
	FFocalLengthInfo FocalLengthInfo;
	ULensFile* LensFile = LensFilePicker.GetLensFile();
	if (LensFile && EvalInputs.bIsValid && LensFile->EvaluateFocalLength(EvalInputs.Focus, EvalInputs.Zoom, FocalLengthInfo))
	{
		// TODO: EvaluateFocalLength should do this scaling internally
		const FVector2D FxFyScale = FVector2D(LensFile->LensInfo.SensorDimensions.X / EvalInputs.Filmback.SensorWidth, LensFile->LensInfo.SensorDimensions.Y / EvalInputs.Filmback.SensorHeight);
		FocalLengthInfo.FxFy = FocalLengthInfo.FxFy * FxFyScale;

		if ((FocalLengthInfo.FxFy.X > UE_KINDA_SMALL_NUMBER) && (FocalLengthInfo.FxFy.Y > UE_KINDA_SMALL_NUMBER))
		{
			// This is how field of view, filmback, and focal length are related:
			//
			// FOVx = 2*atan(1/(2*Fx)) = 2*atan(FilmbackX / (2*FocalLength))
			// => FocalLength = Fx*FilmbackX
			// 
			// FOVy = 2*atan(1/(2*Fy)) = 2*atan(FilmbackY / (2*FocalLength))
			// => FilmbackY = FocalLength / Fy

			// Adjust FocalLength and Filmback to match FxFy (which has already been divided by resolution in pixels)
			const float NewFocalLength = FocalLengthInfo.FxFy.X * EvalInputs.Filmback.SensorWidth * LensFile->LensInfo.SqueezeFactor;

			// TODO: Consider adding an advanced setting to control whether or not sensor height should be modified (if Fx/Fy don't have a fixed aspect ratio)

			// Update the minimum and maximum focal length of the camera (if needed)
			CineCameraComponent->LensSettings.MinFocalLength = FMath::Max(FMath::Min(CineCameraComponent->LensSettings.MinFocalLength, NewFocalLength), 0.01);
			CineCameraComponent->LensSettings.MaxFocalLength = FMath::Max(FMath::Max(CineCameraComponent->LensSettings.MaxFocalLength, NewFocalLength), 0.01);

			CineCameraComponent->SetCurrentFocalLength(NewFocalLength);

			OriginalFocalLength = NewFocalLength;
		}
	}
}

FLensFilePicker ULensComponent::GetLensFilePicker() const
{
	return LensFilePicker;
}

ULensFile* ULensComponent::GetLensFile() const
{
	return LensFilePicker.GetLensFile();
}

void ULensComponent::SetLensFilePicker(FLensFilePicker LensFile)
{
	LensFilePicker = LensFile;

	if (ULensFile* const CachedLensFile = WeakCachedLensFile.Get())
	{
		CachedLensFile->OnLensFileModelChanged().RemoveAll(this);
	}

	WeakCachedLensFile = LensFilePicker.GetLensFile();
	if (ULensFile* const NewLensFile = WeakCachedLensFile.Get())
	{
		if (!NewLensFile->OnLensFileModelChanged().IsBoundToObject(this))
		{
			NewLensFile->OnLensFileModelChanged().AddUObject(this, &ULensComponent::OnLensFileModelChanged);
		}
	}

	if (DistortionStateSource == EDistortionSource::LensFile)
	{
		if (ULensFile* const NewLensFile = LensFilePicker.GetLensFile())
		{
			SetLensModel(NewLensFile->LensInfo.LensModel);
		}
		else
		{
			ClearDistortionState();
		}
	}
}

void ULensComponent::SetLensFile(ULensFile* Lens)
{
	// Automatically sets this to false so the component can use the newly set LensFile directly
	FLensFilePicker NewLensFilePicker;
	NewLensFilePicker.bUseDefaultLensFile = false;
	NewLensFilePicker.LensFile = Lens;

	SetLensFilePicker(NewLensFilePicker);
}

EFIZEvaluationMode ULensComponent::GetFIZEvaluationMode() const
{
	return EvaluationMode;
}

void ULensComponent::SetFIZEvaluationMode(EFIZEvaluationMode Mode)
{
	EvaluationMode = Mode;
}

float ULensComponent::GetOverscanMultiplier() const
{
	if (bScaleOverscan)
	{
		return OverscanMultiplier;
	}
	return 1.0f;
}

void ULensComponent::SetOverscanMultiplier(float Multiplier)
{
	bScaleOverscan = true;
	OverscanMultiplier = Multiplier;
}

EFilmbackOverrideSource ULensComponent::GetFilmbackOverrideSetting() const
{
	return FilmbackOverride;
}

void ULensComponent::SetFilmbackOverrideSetting(EFilmbackOverrideSource Setting)
{
	FilmbackOverride = Setting;
}

FCameraFilmbackSettings ULensComponent::GetCroppedFilmback() const
{
	return CroppedFilmback;
}

void ULensComponent::SetCroppedFilmback(FCameraFilmbackSettings Filmback)
{
	CroppedFilmback = Filmback;
}

const FLensFileEvaluationInputs& ULensComponent::GetLensFileEvaluationInputs() const 
{
	return EvalInputs;
}

void ULensComponent::SetLensFileEvaluationInputs(float InFocus, float InZoom)
{
	EvalInputs.Focus = InFocus;
	EvalInputs.Zoom = InZoom;
	EvalInputs.bIsValid = true;
}

bool ULensComponent::ShouldApplyNodalOffsetOnTick() const
{
	return bApplyNodalOffsetOnTick;
}

void ULensComponent::SetApplyNodalOffsetOnTick(bool bApplyNodalOffset)
{
	bApplyNodalOffsetOnTick = bApplyNodalOffset;
}

bool ULensComponent::WasNodalOffsetAppliedThisTick() const 
{
	return bWasNodalOffsetAppliedThisTick;
}

bool ULensComponent::WasDistortionEvaluated() const
{
	return bWasDistortionEvaluated;
}

float ULensComponent::GetOriginalFocalLength() const 
{
	return OriginalFocalLength;
}

TSubclassOf<ULensModel> ULensComponent::GetLensModel() const
{
	return LensModel;
}

void ULensComponent::SetLensModel(TSubclassOf<ULensModel> Model)
{
	LensModel = Model;

	if (LensModel)
	{
		LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(DistortionState.DistortionInfo.Parameters);

		CreateDistortionHandler();
	}
	else
	{
		DistortionState.DistortionInfo.Parameters.Empty();
	}

	OnLensComponentModelChangedDelegate.Broadcast(LensModel);
}

void ULensComponent::OnLensFileModelChanged(const TSubclassOf<ULensModel>& Model)
{
	if (DistortionStateSource == EDistortionSource::LensFile)
	{
		SetLensModel(Model);
	}
}

FLensDistortionState ULensComponent::GetDistortionState() const
{
	return DistortionState;
}

void ULensComponent::SetDistortionState(FLensDistortionState State)
{
	DistortionState = State;
}

EDistortionSource ULensComponent::GetDistortionSource() const
{
	return DistortionStateSource;
}

void ULensComponent::SetDistortionSource(EDistortionSource Source)
{
	DistortionStateSource = Source;

	// If the new source is a LensFile, update the lens model to match
	if (DistortionStateSource == EDistortionSource::LensFile)
	{
		if (ULensFile* LensFile = LensFilePicker.GetLensFile())
		{
			SetLensModel(LensFile->LensInfo.LensModel);
		}
	}

	ClearDistortionState();
}

bool ULensComponent::ShouldApplyDistortion() const
{
	return bApplyDistortion;
}

void ULensComponent::SetApplyDistortion(bool bApply)
{
	bApplyDistortion = bApply;

	if (bApplyDistortion == false)
	{
		if (UCineCameraComponent* const CineCameraComponent = Cast<UCineCameraComponent>(TargetCameraComponent.GetComponent(GetOwner())))
		{
			CleanupDistortion(CineCameraComponent);
		}

		RestoreCameraOverscanDefaults();
	}
}

bool ULensComponent::GetOverrideCameraOverscan() const
{
	return bOverrideCameraOverscan;
}

void ULensComponent::SetOverrideCameraOverscan(bool bOverride)
{
	bOverrideCameraOverscan = bOverride;

	if (bOverrideCameraOverscan == false)
	{
		RestoreCameraOverscanDefaults();
	}
}

ULensDistortionModelHandlerBase* ULensComponent::GetLensDistortionHandler() const
{
	return LensDistortionHandlerMap.FindRef(LensModel);
}

EDistortionRenderingMode ULensComponent::GetDistortionRenderingMode() const
{
	// If not Preferred, return as specified.
	if (DistortionRenderingMode != EDistortionRenderingMode::Preferred)
	{
		return DistortionRenderingMode;
	}

	// Get the handler for the current lens model
	const ULensDistortionModelHandlerBase* LensDistortionHandler = LensDistortionHandlerMap.FindRef(LensModel);
	if (LensDistortionHandler)
	{
		// Let the lens model decide its preferred rendering mode
		const EDistortionRenderingMode PreferredMode = LensDistortionHandler->GetPreferredRenderingMode();
		
		// If the handler unexpectedly returns Preferred, use default behavior based on IsForwardDistorting()
		if (PreferredMode == EDistortionRenderingMode::Preferred)
		{
			// Reverse distorting models prefer the plugin pass for accuracy, forward distorting models prefer TSR
			return LensDistortionHandler->IsForwardDistorting() 
				? EDistortionRenderingMode::TemporalSuperResolution
				: EDistortionRenderingMode::PassAfterTonemap;
		}
		
		return PreferredMode;
	}

	// Default to plugin pass if no handler is found
	return EDistortionRenderingMode::PassAfterTonemap;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FDistortionHandlerPicker ULensComponent::GetDistortionHandlerPicker() const
{
#if WITH_EDITOR
	return DistortionSource_DEPRECATED;
#else
	return FDistortionHandlerPicker();
#endif //WITH_EDITOR
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void ULensComponent::InitDefaultCamera()
{
	UCineCameraComponent* TargetCamera = Cast<UCineCameraComponent>(TargetCameraComponent.GetComponent(GetOwner()));
	if (!TargetCamera)
	{
		// Find all CineCameraComponents on the same actor as this component and set the first one to be the target
		TInlineComponentArray<UCineCameraComponent*> CineCameraComponents;
		if (AActor* Owner = GetOwner())
		{
			Owner->GetComponents(CineCameraComponents);
		}

		if (CineCameraComponents.Num() > 0)
		{
			TargetCameraComponent.ComponentProperty = CineCameraComponents[0]->GetFName();
			LastCameraComponent = CineCameraComponents[0];
		}
	}
}

float ULensComponent::GetDesqueezedSensorWidth(UCineCameraComponent* const CineCameraComponent) const
{
	if (CineCameraComponent)
	{
		return CineCameraComponent->Filmback.SensorWidth * CineCameraComponent->LensSettings.SqueezeFactor;
	}

	return 1.0f;
}

void ULensComponent::CleanupDistortion(UCineCameraComponent* const CineCameraComponent)
{
	if (bIsDistortionSetup && ensure(CineCameraComponent))
	{
#if WITH_EDITOR
		// If we are in a transaction, we also want to capture the changes made to the camera.
		CineCameraComponent->Modify();
#endif // WITH_EDITOR

		// Remove the last distortion MID that was applied to the target camera component
		if (LastDistortionMID)
		{
			CineCameraComponent->RemoveBlendable(LastDistortionMID);
			LastDistortionMID = nullptr;
		}

		if (UCameraCalibrationSubsystem* SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>())
		{
			if (ACameraActor* CameraActor = Cast<ACameraActor>(GetOwner()))
			{
				SubSystem->ClearLensDistortionSVEState(CameraActor);
			}
		}
	}

	bIsDistortionSetup = false;
	LastAppliedRenderingMode = EDistortionRenderingMode::Preferred;
}

void ULensComponent::ClearDistortionState()
{
	FLensDistortionState ZeroDistortionState;
	if (LensModel)
	{
		LensModel->GetDefaultObject<ULensModel>()->GetDefaultParameterArray(ZeroDistortionState.DistortionInfo.Parameters);
	}
	DistortionState = ZeroDistortionState;
}

void ULensComponent::CreateDistortionHandler()
{
	if (LensModel)
	{
		TObjectPtr<ULensDistortionModelHandlerBase>& Handler = LensDistortionHandlerMap.FindOrAdd(LensModel);
		if (!Handler)
		{
			if (const TSubclassOf<ULensDistortionModelHandlerBase> HandlerClass = ULensModel::GetHandlerClass(LensModel))
			{
				Handler = NewObject<ULensDistortionModelHandlerBase>(this, HandlerClass);
			}
		}
	}
}

void ULensComponent::OnLiveLinkComponentRegistered(ULiveLinkComponentController* LiveLinkComponent)
{
	// Check that the new LiveLinkComponent that was just registered was added to the same actor that this component belongs to
	if (LiveLinkComponent->GetOwner() == GetOwner())
	{
		AddTickPrerequisiteComponent(LiveLinkComponent);

		if (!LiveLinkComponent->OnLiveLinkControllersTicked().IsBoundToObject(this))
		{
			LiveLinkComponent->OnLiveLinkControllersTicked().AddUObject(this, &ULensComponent::ProcessLiveLinkData);
		}
	}
}

void ULensComponent::ProcessLiveLinkData(const ULiveLinkComponentController* const LiveLinkComponent, const FLiveLinkSubjectFrameData& SubjectData)
{
	TSubclassOf<ULiveLinkRole> LiveLinkRole = LiveLinkComponent->GetSubjectRepresentation().Role;

	if (LiveLinkRole.Get()->IsChildOf(ULiveLinkCameraRole::StaticClass()))
	{
		UpdateLiveLinkFIZ(LiveLinkComponent, SubjectData);
	}

	if (LiveLinkRole.Get()->IsChildOf(ULiveLinkTransformRole::StaticClass()))
 	{
 		UpdateTrackedComponent(LiveLinkComponent, SubjectData);
 	}
}

void ULensComponent::UpdateTrackedComponent(const ULiveLinkComponentController* const LiveLinkComponent, const FLiveLinkSubjectFrameData& SubjectData)
{
	const FLiveLinkTransformStaticData* StaticData = SubjectData.StaticData.Cast<FLiveLinkTransformStaticData>();
	const FLiveLinkTransformFrameData* FrameData = SubjectData.FrameData.Cast<FLiveLinkTransformFrameData>();

	check(StaticData && FrameData);

	// Find the transform controller in the LiveLink component's controller map
	if (const TObjectPtr<ULiveLinkControllerBase>* TransformControllerPtr = LiveLinkComponent->ControllerMap.Find(ULiveLinkTransformRole::StaticClass()))
	{
		if (ULiveLinkTransformController* TransformController = Cast<ULiveLinkTransformController>(*TransformControllerPtr))
		{
			// Check LiveLink static data to ensure that location and rotation were supported by this subject
			bool bIsLocationRotationSupported = (StaticData->bIsLocationSupported && StaticData->bIsRotationSupported);

			// Check the transform controller usage flags to ensure that location and rotation were supported by the controller
			bIsLocationRotationSupported = bIsLocationRotationSupported && (TransformController->TransformData.bUseLocation && TransformController->TransformData.bUseRotation);

			if (bIsLocationRotationSupported)
			{
				TrackedComponent = Cast<USceneComponent>(TransformController->GetAttachedComponent());
				if (TrackedComponent.IsValid())
				{
					TrackedComponentName = TrackedComponent.Get()->GetName();
				}
				// At this point, we know that the tracked component's transform was just set by LiveLink, and therefore has no nodal offset applied
				bWasNodalOffsetAppliedThisTick = false;
			}
		}
	}
}

void ULensComponent::UpdateLiveLinkFIZ(const ULiveLinkComponentController* const LiveLinkComponent, const FLiveLinkSubjectFrameData& SubjectData)
{
	const FLiveLinkCameraStaticData* StaticData = SubjectData.StaticData.Cast<FLiveLinkCameraStaticData>();
	const FLiveLinkCameraFrameData* FrameData = SubjectData.FrameData.Cast<FLiveLinkCameraFrameData>();
	
	check(StaticData && FrameData);

	// Check that the camera component being controlled by the input LiveLink component matches our target camera
	if (const TObjectPtr<ULiveLinkControllerBase>* CameraControllerPtr = LiveLinkComponent->ControllerMap.Find(ULiveLinkCameraRole::StaticClass()))
	{
		if (ULiveLinkControllerBase* CameraController = *CameraControllerPtr)
		{
			if (CameraController->GetAttachedComponent() != TargetCameraComponent.GetComponent(GetOwner()))
			{
				return;
			}
		}
	}

	if (StaticData->bIsFocusDistanceSupported && StaticData->bIsFocalLengthSupported)
	{
		LiveLinkFocus = FrameData->FocusDistance;
		LiveLinkZoom = FrameData->FocalLength;

		LiveLinkIris = FrameData->Aperture;

		bWasLiveLinkFIZUpdated = true;
	}
}

void ULensComponent::UpdateLensFileEvaluationInputs(UCineCameraComponent* CineCameraComponent)
{
	EvalInputs.bIsValid = false;

	// Query for the focus and zoom inputs to use to evaluate the LensFile. The source of these inputs will depend on the evaluation mode.
	if (EvaluationMode == EFIZEvaluationMode::UseLiveLink)
	{
		if (bWasLiveLinkFIZUpdated)
		{
			EvalInputs.Focus = LiveLinkFocus;
			EvalInputs.Iris = LiveLinkIris;
			EvalInputs.Zoom = LiveLinkZoom;
			EvalInputs.bIsValid = true;
		}
	}
	else if (EvaluationMode == EFIZEvaluationMode::UseCameraSettings)
	{
		EvalInputs.Focus = CineCameraComponent->CurrentFocusDistance;
		EvalInputs.Iris = CineCameraComponent->CurrentAperture;
		// The camera's focal length might have been altered by applying an overscan FOV. However, the LensFile needs to be evaluated with the non-overscanned value
		EvalInputs.Zoom = OriginalFocalLength;
		EvalInputs.bIsValid = true;
	}
	else if ((EvaluationMode == EFIZEvaluationMode::UseRecordedValues) || (EvaluationMode == EFIZEvaluationMode::Manual))
	{
		// Do nothing, the values for EvalInputs.Focus and EvalInputs.Zoom are already valid
		EvalInputs.bIsValid = true;
	}
}

void ULensComponent::UpdateCameraFilmback(UCineCameraComponent* CineCameraComponent)
{	
	switch (FilmbackOverride)
	{
		case EFilmbackOverrideSource::LensFile:
		{	
			if (ULensFile* LensFile = LensFilePicker.GetLensFile())
			{
				CineCameraComponent->Filmback.SensorWidth = LensFile->LensInfo.SensorDimensions.X;
				CineCameraComponent->Filmback.SensorHeight = LensFile->LensInfo.SensorDimensions.Y;
				CineCameraComponent->LensSettings.SqueezeFactor = LensFile->LensInfo.SqueezeFactor;
			}
			break;
		}
		case EFilmbackOverrideSource::CroppedFilmbackSetting:
		{
			CineCameraComponent->Filmback = CroppedFilmback;

			if (ULensFile* LensFile = LensFilePicker.GetLensFile())
			{
				CineCameraComponent->LensSettings.SqueezeFactor = LensFile->LensInfo.SqueezeFactor;
			}
			break;
		}
		case EFilmbackOverrideSource::DoNotOverride:
		{
			break; // Do Nothing
		}
		default:
		{
			ensureMsgf(false, TEXT("Unhandled EFilmbackOverrideSource type. Update this switch statement."));
			break; 
		}
	}

	EvalInputs.Filmback = CineCameraComponent->Filmback;
}
