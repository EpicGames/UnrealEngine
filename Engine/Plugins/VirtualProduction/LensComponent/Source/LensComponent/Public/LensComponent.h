// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"

#include "CameraCalibrationTypes.h"
#include "LensDistortionModelHandlerBase.h"
#include "LensFile.h"
#include "LiveLinkComponentController.h"

#include "LensComponent.generated.h"

#define UE_API LENSCOMPONENT_API

class FLensComponentDetailCustomization;
class UCineCameraComponent;


DECLARE_MULTICAST_DELEGATE_OneParam(FOnLensComponentModelChanged, const TSubclassOf<ULensModel>&);


/** Mode that controls where FIZ inputs are sourced from and how they are used to evaluate the LensFile */
UENUM(BlueprintType)
enum class EFIZEvaluationMode : uint8
{
	/** Evaluate the Lens File with the latest FIZ data received from LiveLink */
	UseLiveLink,
	/** Evaluate the Lens File using the current FIZ settings of the target camera */
	UseCameraSettings,
	/** Evaluate the Lens File using values recorded in a level sequence (set automatically when the sequence is opened) */
	UseRecordedValues,
	/** Evaluate the Lens File using values set directly in the details panel or via BP/scripting */
	Manual,
	/** Do not evaluate the Lens File */
	DoNotEvaluate,
};

/** Controls whether this component can override the camera's filmback, and if so, which override to use */
UENUM(BlueprintType)
enum class EFilmbackOverrideSource : uint8
{
	/** Override the camera's filmback using the sensor dimensions recorded in the LensInfo of the LensFile */
	LensFile,
	/** Override the camera's filmback using the CroppedFilmback setting below */
	CroppedFilmbackSetting,
	/** Do not override the camera's filmback */
	DoNotOverride,
};

/** Specifies from where the distortion state information comes */
UENUM(BlueprintType)
enum class EDistortionSource : uint8
{
	/** Distortion state is evaluated using the LensFile */
	LensFile,
	/** Distortion state is inputted directly from a LiveLink subject */
	LiveLinkLensSubject,
	/** Distortion state is set manually by the user using the Distortion State setting below */
	Manual,
};

/** Component for applying a post-process lens distortion effect to a CineCameraComponent on the same actor */
UCLASS(MinimalAPI, HideCategories=(Tags, Activation, Cooking, AssetUserData, Collision), meta=(BlueprintSpawnableComponent))
class ULensComponent : public UActorComponent
{
	GENERATED_BODY()

	friend FLensComponentDetailCustomization;

public:
	UE_API ULensComponent();

	//~ Begin UActorComponent interface
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	UE_API virtual void DestroyComponent(bool bPromoteChildren = false) override;
	//~ End UActorComponent interface

	//~ Begin UObject interface
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostLoad() override;

	UE_API virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR
	//~ End UObject interface

	/** Returns the delegate that is triggered when the LensModel changes */
	FOnLensComponentModelChanged& OnLensComponentModelChanged() 
	{ 
		return OnLensComponentModelChangedDelegate; 
	}

	/** Get the LensFile picker used by this component */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API FLensFilePicker GetLensFilePicker() const;

	/** Get the LensFile used by this component */
	UFUNCTION(BlueprintPure, Category="Lens Component")
	UE_API ULensFile* GetLensFile() const;

	/** Set the LensFile picker used by this component */
	UFUNCTION(BlueprintCallable, Category = "Lens Component")
	UE_API void SetLensFilePicker(FLensFilePicker LensFile);

	/** Set the LensFile used by this component */
	UFUNCTION(BlueprintCallable, Category="Lens Component")
	UE_API void SetLensFile(ULensFile* LensFile);

	/** Get the evaluation mode used to evaluate the LensFile */
	UFUNCTION(BlueprintPure, Category="Lens Component", meta = (DisplayName = "Get FIZ Evaluation Mode"))
	UE_API EFIZEvaluationMode GetFIZEvaluationMode() const;

	/** Set the evaluation mode used to evaluate the LensFile */
	UFUNCTION(BlueprintCallable, Category="Lens Component", meta=(DisplayName="Set FIZ Evaluation Mode"))
	UE_API void SetFIZEvaluationMode(EFIZEvaluationMode Mode);

	/** Get the evaluation mode used to evaluate the LensFile */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API float GetOverscanMultiplier() const;

	/** Set the LensFile used by this component */
	UFUNCTION(BlueprintCallable, Category = "Lens Component")
	UE_API void SetOverscanMultiplier(float Multiplier);

	/** Get the filmback override setting */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API EFilmbackOverrideSource GetFilmbackOverrideSetting() const;

	/** Set the filmback override setting */
	UFUNCTION(BlueprintCallable, Category = "Lens Component")
	UE_API void SetFilmbackOverrideSetting(EFilmbackOverrideSource Setting);

	/** Get the cropped filmback (only relevant if the filmback override setting is set to use the CroppedFilmback */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API FCameraFilmbackSettings GetCroppedFilmback() const;

	/** Set the cropped filmback (only relevant if the filmback override setting is set to use the CroppedFilmback */
	UFUNCTION(BlueprintCallable, Category = "Lens Component")
	UE_API void SetCroppedFilmback(FCameraFilmbackSettings Filmback);

	/** Returns true if nodal offset will be automatically applied during this component's tick, false otherwise */
	UFUNCTION(BlueprintPure, Category="Lens Component")
	UE_API bool ShouldApplyNodalOffsetOnTick() const;

	/** Set whether nodal offset should be automatically applied during this component's tick */
	UFUNCTION(BlueprintCallable, Category="Lens Component")
	UE_API void SetApplyNodalOffsetOnTick(bool bApplyNodalOffset);

	/** Get the distortion source setting */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API EDistortionSource GetDistortionSource() const;

	/** Set the distortion source setting */
	UFUNCTION(BlueprintCallable, Category = "Lens Component")
	UE_API void SetDistortionSource(EDistortionSource Source);

	/** Whether distortion should be applied to the target camera */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API bool ShouldApplyDistortion() const;

	/** Set whether distortion should be applied to the target camera */
	UFUNCTION(BlueprintCallable, Category = "Lens Component")
	UE_API void SetApplyDistortion(bool bApply);

	/** Get whether the camera overscan is overriden. */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API bool GetOverrideCameraOverscan() const;

	/** Set whether the camera overscan should be overriden. */
	UFUNCTION(BlueprintCallable, Category = "Lens Component")
	UE_API void SetOverrideCameraOverscan(bool bOverride);

	/** Get the current lens model */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API TSubclassOf<ULensModel> GetLensModel() const;

	/** Set the current lens model */
	UFUNCTION(BlueprintCallable, Category = "Lens Component")
	UE_API void SetLensModel(TSubclassOf<ULensModel> Model);

	/** Get the current distortion state */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API FLensDistortionState GetDistortionState() const;

	/** Set the current distortion state */
	UFUNCTION(BlueprintCallable, Category = "Lens Component")
	UE_API void SetDistortionState(FLensDistortionState State);

	/** Reset the distortion state to defaults to represent "no distortion" */
	UFUNCTION(BlueprintCallable, Category = "Lens Component")
	UE_API void ClearDistortionState();

	/** Get the original (not adjusted for overscan) focal length of the camera */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API float GetOriginalFocalLength() const;

	/** Get the data used by this component to evaluate the LensFile */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API const FLensFileEvaluationInputs& GetLensFileEvaluationInputs() const;

	/** Set the data used by this component to evaluate the LensFile */
	UFUNCTION(BlueprintCallable, Category = "Lens Component")
	UE_API void SetLensFileEvaluationInputs(float InFocus, float InZoom);

	/** Returns true if nodal offset was applied during the current tick, false otherwise */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API bool WasNodalOffsetAppliedThisTick() const;

	/** Returns true if distortion was evaluated this tick */
	UFUNCTION(BlueprintPure, Category = "Lens Component")
	UE_API bool WasDistortionEvaluated() const;

	/** 
	 * Manually apply nodal offset to the specified component. 
	 * If bUseManualInputs is true, the input Focus and Zoom values will be used to evaluate the LensFile .
	 * If bUseManualInputs is false, the LensFile be will evaluated based on the Lens Component's evaluation mode.
	 */
	UFUNCTION(BlueprintCallable, Category="Lens Component", meta=(AdvancedDisplay=1))
	UE_API void ApplyNodalOffset(USceneComponent* ComponentToOffset, bool bUseManualInputs = false, float ManualFocusInput = 0.0f, float ManualZoomInput = 0.0f);

public:
	/** Returns the LensDistortionHandler in use for the current LensModel */
	UFUNCTION(BlueprintCallable, Category = "Lens Component")
	UE_API ULensDistortionModelHandlerBase* GetLensDistortionHandler() const;

	/** Reset the tracked component back to its original tracked pose and reapply nodal offset to it by re-evaluating the LensFile */
	UE_API void ReapplyNodalOffset();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "This property is deprecated. Use GetDistortionSource() and SetDistortionSource() instead.")
	UE_API FDistortionHandlerPicker GetDistortionHandlerPicker() const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:
	/** Evaluate the LensFile for nodal offset (using the current evaluation mode) and apply it to the latest component to offset */
	void ApplyNodalOffset();

	/** Evaluate the focal length from the LensFile and applies the calibrated value to the camera */
	UE_API void EvaluateFocalLength(UCineCameraComponent* CineCameraComponent);

	/** If TargetCameraComponent is not set, initialize it to the first CineCameraComponent on the same actor as this component */
	UE_API void InitDefaultCamera();

	/** Remove the last distortion MID applied to the input CineCameraComponent and reset its FOV to use no overscan */
	UE_API void CleanupDistortion(UCineCameraComponent* CineCameraComponent);

	/** Register a new lens distortion handler with the camera calibration subsystem using the selected lens file */
	UE_API void CreateDistortionHandler();

	/** Register to the new LiveLink component's callback to be notified when its controller map changes */
	UE_API void OnLiveLinkComponentRegistered(ULiveLinkComponentController* LiveLinkComponent);
	
	/** Triggered when the LensFile model changes */
	UE_API void OnLensFileModelChanged(const TSubclassOf<ULensModel>& Model);

	/** Callback executed when a LiveLink component on the same actor ticks */
	UE_API void ProcessLiveLinkData(const ULiveLinkComponentController* const LiveLinkComponent, const FLiveLinkSubjectFrameData& SubjectData);

	/** Inspects the subject data and LiveLink transform controller to determine which component (if any) had tracking data applied to it */
	UE_API void UpdateTrackedComponent(const ULiveLinkComponentController* const LiveLinkComponent, const FLiveLinkSubjectFrameData& SubjectData);

	/** Inspects the subject data and LiveLink camera controller cache the FIZ that was input for the target camera */
	UE_API void UpdateLiveLinkFIZ(const ULiveLinkComponentController* const LiveLinkComponent, const FLiveLinkSubjectFrameData& SubjectData);

	/** Updates the focus and zoom inputs that will be used to evaluate the LensFile based on the evaluation mode */
	UE_API void UpdateLensFileEvaluationInputs(UCineCameraComponent* CineCameraComponent);

	/** Updates the camera's filmback based on the filmback override settings */
	UE_API void UpdateCameraFilmback(UCineCameraComponent* CineCameraComponent);

	/** Returns the sensor width of the input CineCamera, factoring in its squeeze factor */
	UE_API float GetDesqueezedSensorWidth(UCineCameraComponent* const CineCameraComponent) const;

	/** Get the original transform of the tracked component (rebuilt from the location and rotation vectors) */
	UE_API FTransform GetOriginalTrackedComponentTransform();

	/** Set the original transform of the tracked component (and also the location and rotation vectors) */
	UE_API void SetOriginalTrackedComponentTransform(const FTransform& NewTransform);

	/** Convenience function to restore the camera overscan defaults. */
	void RestoreCameraOverscanDefaults();

protected:
	/** Lens File used to drive distortion with current camera settings */
	UPROPERTY(EditAnywhere, Category="Lens File", meta=(ShowOnlyInnerProperties))
	FLensFilePicker LensFilePicker;

	/** Specify how the Lens File should be evaluated */
	UPROPERTY(EditAnywhere, Category="Lens File")
	EFIZEvaluationMode EvaluationMode = EFIZEvaluationMode::UseLiveLink;

	/** The CineCameraComponent on which to apply the post-process distortion effect */
	UPROPERTY(EditInstanceOnly, AdvancedDisplay, Category="Lens File", meta=(UseComponentPicker, AllowedClasses="/Script/CinematicCamera.CineCameraComponent"))
	FComponentReference TargetCameraComponent;

	/** Inputs to LensFile evaluation */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Lens File")
	FLensFileEvaluationInputs EvalInputs;

	/** Specifies from where the distortion state information comes */
	UPROPERTY(EditAnywhere, Category = "Distortion", meta = (DisplayName = "Distortion Source"))
	EDistortionSource DistortionStateSource = EDistortionSource::LensFile;

	/** Whether or not to apply distortion to the target camera component */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = ShouldApplyDistortion, Setter = SetApplyDistortion, Category = "Distortion")
	bool bApplyDistortion = false;

	/** Specifies how the distortion should be rendered in the post-processing pipeline */
	UPROPERTY(EditAnywhere, Category = "Distortion", meta = (EditCondition = "bApplyDistortion"))
	EDistortionRenderingMode DistortionRenderingMode = EDistortionRenderingMode::Preferred;

	/**
	 * If checked, the camera's overscan value will be driven by the lens component to automatically compensate for distortion. 
	 * The camera's overscan crop property will also be driven based on the distortion rendering mode:
	 *   Disabled for Post Process Material or pass after tonemap
	 *   Enabled for Temporal Super Resolution
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = GetOverrideCameraOverscan, Setter = SetOverrideCameraOverscan, Category = "Distortion")
	bool bOverrideCameraOverscan = true;

	/** The current lens model used for distortion */
	UPROPERTY(EditAnywhere, Category = "Distortion", meta = (EditCondition = "DistortionStateSource == EDistortionSource::Manual"))
	TSubclassOf<ULensModel> LensModel;

	/** The current distortion state */
	UPROPERTY(Interp, EditAnywhere, Category = "Distortion", meta = (EditCondition = "DistortionStateSource == EDistortionSource::Manual"))
	FLensDistortionState DistortionState;

	/** Whether to scale the computed overscan by the overscan percentage */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = "Distortion", meta = (InlineEditConditionToggle))
	bool bScaleOverscan = false;

	/** The percentage of the computed overscan that should be applied to the target camera */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category = "Distortion", meta = (EditCondition = "bScaleOverscan", ClampMin = "0.0", ClampMax = "2.0"))
	float OverscanMultiplier = 1.0f;

	/** Controls whether this component can override the camera's filmback, and if so, which override to use */
	UPROPERTY(EditAnywhere, Category = "Filmback")
	EFilmbackOverrideSource FilmbackOverride = EFilmbackOverrideSource::DoNotOverride;

	/** Cropped filmback to use if the filmback override settings are set to use it */
	UPROPERTY(EditAnywhere, Category = "Filmback", meta=(EditCondition="FilmbackOverride == EFilmbackOverrideSource::CroppedFilmbackSetting"))
	FCameraFilmbackSettings CroppedFilmback;

	/** 
	 * If checked, nodal offset will be applied automatically when this component ticks. 
	 * Set to false if nodal offset needs to be manually applied at some other time (via Blueprints).
	 */
	UPROPERTY(EditAnywhere, Category="Nodal Offset")
	bool bApplyNodalOffsetOnTick = true;

	/*
	 * Location and Rotation of the TrackedComponent prior to nodal offset being applied 
	 * Note: These are marked Interp so that they will be recorded in a level sequence to support re-applying nodal offset
	 * However, recording of FTransform properties is not currently supported by the transform track recorder.
	 * FRotator and FQuat are also not supported by the basic property track recorder, but FVector is, so we use that for both location and rotation.
	 */
	UPROPERTY(Interp, Category = "Nodal Offset", meta = (EditCondition=false, EditConditionHides))
	FVector OriginalTrackedComponentLocation;
	UPROPERTY(Interp, Category = "Nodal Offset", meta = (EditCondition=false, EditConditionHides))
	FVector OriginalTrackedComponentRotation;

	/** Serialized transform of the TrackedComponent prior to nodal offset being applied */
	UPROPERTY(Interp, VisibleAnywhere, AdvancedDisplay, Category = "Nodal Offset")
	FTransform OriginalTrackedComponentTransform;

	/** Whether a distortion effect is currently being applied to the target camera component */
	UPROPERTY()
	bool bIsDistortionSetup = false;

	/** Focal length of the target camera before any overscan has been applied */
	UPROPERTY()
	float OriginalFocalLength = 35.0f;

	/** Cached MID last applied to the target camera */
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> LastDistortionMID = nullptr;

	/** Track the last applied rendering mode to detect changes when using Preferred mode */
	EDistortionRenderingMode LastAppliedRenderingMode = EDistortionRenderingMode::Preferred;

	/** Cached most recent target camera, used to clean up the old camera when the user changes the target */
	UPROPERTY()
	TObjectPtr<UCineCameraComponent> LastCameraComponent = nullptr;

	/** Map of lens models to handlers */
	UPROPERTY(Transient)
	TMap<TSubclassOf<ULensModel>, TObjectPtr<ULensDistortionModelHandlerBase>> LensDistortionHandlerMap;

	/** Scene component that should have nodal offset applied */
	UPROPERTY(Transient)
	TWeakObjectPtr<USceneComponent> TrackedComponent;

	/** Serialized name of the TrackedComponent, used to determine which component to re-apply nodal offset to in spawnables */
	UPROPERTY()
	FString TrackedComponentName;

	/** Weak pointer to the currently set LensFile. Useful to unregister events from the previous LensFile when the asset is swapped out */
	TWeakObjectPtr<ULensFile> WeakCachedLensFile;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "This property has been deprecated. The LensDistortion component no longer tracks the attached camera's original rotation.")
	UPROPERTY()
	FRotator OriginalCameraRotation_DEPRECATED;

	UE_DEPRECATED(5.1, "This property has been deprecated. The LensDistortion component no longer tracks the attached camera's original location.")
	UPROPERTY()
	FVector OriginalCameraLocation_DEPRECATED;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.1, "This property has been deprecated. The handler picker is no longer used to identify a distortion handler. Use the LensDistortionHandlerMap to get a handler for the current LensModel.")
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="This property has been deprecated. Use GetDistortionSource() and SetDistortionSource() instead."))
	FDistortionHandlerPicker DistortionSource_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif //WITH_EDITORONLY_DATA

private:
	/** Latest LiveLink FIZ data, used to evaluate the LensFile */
	float LiveLinkFocus = 0.0f;
	float LiveLinkIris = 0.0f;
	float LiveLinkZoom = 0.0f;

	/** Whether LiveLink FIZ was received this tick */
	bool bWasLiveLinkFIZUpdated = false;

	/** Whether distortion was evaluated this tick */
	bool bWasDistortionEvaluated = false;

	/** Whether or not nodal offset was applied to a tracked component this tick */
	bool bWasNodalOffsetAppliedThisTick = false;

	/** Latest focal length of the target camera, used to track external changes to the original (no overscan applied) focal length */
	float LastFocalLength = -1.0f;

	/** Delegate that is triggered when the LensModel changes */
	FOnLensComponentModelChanged OnLensComponentModelChangedDelegate;

	/** Resolves the distortion rendering mode to the actual mode that will be used (resolving Preferred if necessary) */
	EDistortionRenderingMode GetDistortionRenderingMode() const;
};

#undef UE_API
