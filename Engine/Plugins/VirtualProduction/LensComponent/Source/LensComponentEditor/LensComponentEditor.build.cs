// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LensComponentEditor : ModuleRules
	{
		public LensComponentEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CameraCalibrationCore",
					"CameraCalibrationCoreEditor",
					"Core",
					"CoreUObject",
					"Engine",
					"LensComponent",
					"LevelSequence",
					"MovieScene",
					"MovieSceneTools",
					"Sequencer",
					"Slate",
					"SlateCore",
					"TakeTrackRecorders",
					"UnrealEd",
				}
			);
		}
	}
}
