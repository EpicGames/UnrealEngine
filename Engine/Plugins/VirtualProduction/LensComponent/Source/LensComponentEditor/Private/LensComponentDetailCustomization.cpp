// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensComponentDetailCustomization.h"

#include "DetailLayoutBuilder.h"
#include "LensDistortionStateDetailCustomization.h"

FLensComponentDetailCustomization::~FLensComponentDetailCustomization()
{
	if (ULensComponent* LensComponent = WeakLensComponent.Get())
	{
		LensComponent->OnLensComponentModelChanged().RemoveAll(this);
	}
}

void FLensComponentDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailLayout = &DetailBuilder;

	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailBuilder.GetSelectedObjects();

 	if (SelectedObjects.Num() != 1)
 	{
 		return;
 	}

	if (ULensComponent* CustomizedLensComponent = Cast<ULensComponent>(SelectedObjects[0].Get()))
	{
		WeakLensComponent = CustomizedLensComponent;

		if (!CustomizedLensComponent->OnLensComponentModelChanged().IsBoundToObject(this))
		{
			CustomizedLensComponent->OnLensComponentModelChanged().AddSP(this, &FLensComponentDetailCustomization::OnLensModelChanged);
		}

		TSharedRef<IPropertyHandle> LensModelPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULensComponent, LensModel));
		LensModelPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FLensComponentDetailCustomization::OnLensModelChanged));

		const TSubclassOf<ULensModel> LensModel = CustomizedLensComponent->GetLensModel();

		FOnGetPropertyTypeCustomizationInstance DistortionStateCustomizationDelegate = FOnGetPropertyTypeCustomizationInstance::CreateLambda(
			[LensModel]
			{
				return MakeShared<FLensDistortionStateDetailCustomization>(LensModel);
			});

		DetailBuilder.RegisterInstancedCustomPropertyTypeLayout(FLensDistortionState::StaticStruct()->GetFName(), DistortionStateCustomizationDelegate);
	}
}

void FLensComponentDetailCustomization::OnLensModelChanged()
{
	DetailLayout->ForceRefreshDetails();
}

void FLensComponentDetailCustomization::OnLensModelChanged(const TSubclassOf<ULensModel>& LensModel)
{
	DetailLayout->ForceRefreshDetails();
}
