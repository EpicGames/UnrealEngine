// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

#include "LensComponent.h"

/**
* Customizes a ULensComponent details
*/
class FLensComponentDetailCustomization : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FLensComponentDetailCustomization>();
	}

	~FLensComponentDetailCustomization();

	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface

protected:
	/** Forces the details panel to refresh when the lens model changes */
	void OnLensModelChanged();

	/** Forces the details panel to refresh when the lens model changes */
	void OnLensModelChanged(const TSubclassOf<ULensModel>& LensModel);

protected:
	/** Keep a reference to force refresh the layout */
	IDetailLayoutBuilder* DetailLayout = nullptr;

	/** Weak pointer to the lens component being edited */
	TWeakObjectPtr<ULensComponent> WeakLensComponent;
};
