// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"

#include "Editor.h"
#include "Features/IModularFeatures.h"
#include "ISequencerModule.h"
#include "LensComponent.h"
#include "LensComponentDetailCustomization.h"
#include "Modules/ModuleManager.h"
#include "MovieScene/LensComponentTrackEditor.h"
#include "MovieScene/MovieSceneLensComponentTrackRecorder.h"
#include "PropertyEditorModule.h"

static const FName MovieSceneTrackRecorderFactoryName("MovieSceneTrackRecorderFactory");

class FLensComponentEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		PropertyEditorModule.RegisterCustomClassLayout(
			ULensComponent::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateStatic(&FLensComponentDetailCustomization::MakeInstance)
		);

		// Register Track Editor for LensComponent track
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		CreateLensComponentTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FLensComponentTrackEditor::CreateTrackEditor));

		// Register modular feature for LensComponent track recorder factory
		IModularFeatures::Get().RegisterModularFeature(MovieSceneTrackRecorderFactoryName, &LensComponentTrackRecorderFactory);
	}

	virtual void ShutdownModule() override
	{
		if (!IsEngineExitRequested() && GEditor && UObjectInitialized())
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomClassLayout(ULensComponent::StaticClass()->GetFName());
		}
	}
	//~ End IModuleInterface interface

private:
	FDelegateHandle CreateLensComponentTrackEditorHandle;

	FMovieSceneLensComponentTrackRecorderFactory LensComponentTrackRecorderFactory;
};

IMPLEMENT_MODULE(FLensComponentEditorModule, LensComponentEditor);
