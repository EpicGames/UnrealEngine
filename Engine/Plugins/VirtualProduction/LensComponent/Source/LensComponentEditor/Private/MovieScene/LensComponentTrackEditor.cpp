// Copyright Epic Games, Inc. All Rights Reserved.

#include "LensComponentTrackEditor.h"

#include "LevelSequence.h"

#define LOCTEXT_NAMESPACE "LensComponentTrackEditor"

TSharedRef<ISequencerTrackEditor> FLensComponentTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShared<FLensComponentTrackEditor>(InSequencer);
}

FText FLensComponentTrackEditor::GetDisplayName() const
{
	return LOCTEXT("LensComponentTrackEditor_DisplayName", "Lens");
}

bool FLensComponentTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	if (!InSequence || InSequence->IsTrackSupported(UMovieSceneLensComponentTrack::StaticClass()) == ETrackSupport::NotSupported)
	{
		return false;
	}

	return InSequence->IsA(ULevelSequence::StaticClass());
}

bool FLensComponentTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UMovieSceneLensComponentTrack::StaticClass());
}

#undef LOCTEXT_NAMESPACE
