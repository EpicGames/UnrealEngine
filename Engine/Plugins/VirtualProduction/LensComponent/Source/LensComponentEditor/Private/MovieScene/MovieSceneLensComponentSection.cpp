// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLensComponentSection.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "Evaluation/MovieSceneEvaluationState.h"
#include "Evaluation/MovieScenePreAnimatedState.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSource.h"
#include "MovieSceneExecutionToken.h"
#include "MovieSceneTimeHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLensComponentSection)

#define LOCTEXT_NAMESPACE "MovieSceneLensComponentSection"

struct FPreAnimatedLensFileTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const
	{
		struct FLensFileToken : IMovieScenePreAnimatedToken
		{
			FLensFileToken(ULensComponent* LensComponent)
			{
				// Save the original values of these settings
				FLensFilePicker LensFilePicker = LensComponent->GetLensFilePicker();
				bOriginalUseDefaultLensFile = LensFilePicker.bUseDefaultLensFile;
				OriginalLensFile = LensFilePicker.GetLensFile();
				OriginalEvalMode = LensComponent->GetFIZEvaluationMode();
			}

			virtual void RestoreState(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params)
			{
				if (ULensComponent* LensComponent = CastChecked<ULensComponent>(&Object))
				{
					// Restore the original values of these settings
					LensComponent->SetLensFilePicker({ bOriginalUseDefaultLensFile, OriginalLensFile.Get() });
					LensComponent->SetFIZEvaluationMode(OriginalEvalMode);
				}
			}

			bool bOriginalUseDefaultLensFile = false;
			TWeakObjectPtr<ULensFile> OriginalLensFile;
			EFIZEvaluationMode OriginalEvalMode = EFIZEvaluationMode::UseRecordedValues;
		};

		return FLensFileToken(CastChecked<ULensComponent>(&Object));
	}
	static FMovieSceneAnimTypeID GetAnimTypeID()
	{
		return TMovieSceneAnimTypeID<FPreAnimatedLensFileTokenProducer>();
	}
};

void UMovieSceneLensComponentSection::Initialize(ULensComponent* Component)
{
	// Cache a pointer to the recorded component to retrieve its data later during RecordFrame()
	RecordedComponent = Component;

	// Cache the Lens Model used by the recorded lens component
	LensModelClass = RecordedComponent->GetLensModel();

	// Duplicate the recorded LensFile asset and save the duplicate in this section
 	if (ULensFile* LensFile = Component->GetLensFile())
 	{
 		const FName DuplicateName = FName(*FString::Format(TEXT("{0}_Cached"), { LensFile->GetFName().ToString() }));
 		CachedLensFile = Cast<ULensFile>(StaticDuplicateObject(LensFile, this, DuplicateName, RF_AllFlags & ~RF_Transient));
 	}

	// Create the channels that will be used to record the distortion state
	CreateChannelProxy();
}

void UMovieSceneLensComponentSection::PostEditImport()
{
	Super::PostEditImport();

	UpdateChannelProxy();
}

void UMovieSceneLensComponentSection::PostLoad()
{
	Super::PostLoad();

	UpdateChannelProxy();
}

#if WITH_EDITOR
void UMovieSceneLensComponentSection::AddChannelWithEditor(FMovieSceneChannelProxyData& ChannelProxyData, FMovieSceneFloatChannel& Channel, FText GroupName, FText ChannelName, int32 SortOrder)
{
	FMovieSceneChannelMetaData ChannelEditorData(FName(*(ChannelName.ToString())), ChannelName, GroupName);
	ChannelEditorData.SortOrder = SortOrder;
	ChannelEditorData.bCanCollapseToTrack = false;

	ChannelProxyData.Add(Channel, ChannelEditorData, TMovieSceneExternalValue<float>());
}
#endif // WITH_EDITOR

void UMovieSceneLensComponentSection::CreateChannelProxy()
{
	// Add channels for distortion parameters
	if (LensModelClass)
	{
		if (const ULensModel* const LensModel = LensModelClass->GetDefaultObject<ULensModel>())
		{
			const int32 ParameterCount = LensModel->GetNumParameters();
			DistortionParameterChannels.SetNum(ParameterCount);
		}
	}

	// Add channels for FxFy
	FxFyChannels.SetNum(2);

	// Add channels for Image Center
	ImageCenterChannels.SetNum(2);

	UpdateChannelProxy();
}

void UMovieSceneLensComponentSection::UpdateChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

	// Add channels for distortion parameters
	if (LensModelClass)
	{
		if (const ULensModel* const LensModel = LensModelClass->GetDefaultObject<ULensModel>())
		{
			const int32 ParameterCount = LensModel->GetNumParameters();
			if (ParameterCount == DistortionParameterChannels.Num())
			{
#if WITH_EDITOR
				TArray<FText> ParameterDisplayNames = LensModel->GetParameterDisplayNames();
				const FText ParameterGroupName = LOCTEXT("ParameterGroupName", "Distortion Parameters");
#endif // WITH_EDITOR

				for (int32 Index = 0; Index < ParameterCount; ++Index)
				{
#if WITH_EDITOR
					AddChannelWithEditor(Channels, DistortionParameterChannels[Index], ParameterGroupName, ParameterDisplayNames[Index], Index);
#else
					Channels.Add(DistortionParameterChannels[Index]);
#endif // WITH_EDITOR
				}
			}
		}
	}

	// Add channels for FxFy
	if (FxFyChannels.Num() == 2)
	{
#if WITH_EDITOR
		const FText FxFyGroupName = LOCTEXT("FxFyGroupName", "FxFy");
		constexpr int32 FxFySortOrder = 1000;
		AddChannelWithEditor(Channels, FxFyChannels[0], FxFyGroupName, LOCTEXT("FxChannelName", "Fx"), FxFySortOrder + 0);
		AddChannelWithEditor(Channels, FxFyChannels[1], FxFyGroupName, LOCTEXT("FyChannelName", "Fy"), FxFySortOrder + 1);
#else
		Channels.Add(FxFyChannels[0]);
		Channels.Add(FxFyChannels[1]);
#endif // WITH_EDITOR
	}

	// Add channels for Image Center
	if (ImageCenterChannels.Num() == 2)
	{
#if WITH_EDITOR
		const FText ImageCenterGroupName = LOCTEXT("CxCyGroupName", "Image Center");
		constexpr int32 ImageCenterSortOrder = 2000;
		AddChannelWithEditor(Channels, ImageCenterChannels[0], ImageCenterGroupName, LOCTEXT("CxChannelName", "Cx"), ImageCenterSortOrder + 0);
		AddChannelWithEditor(Channels, ImageCenterChannels[1], ImageCenterGroupName, LOCTEXT("CyChannelName", "Cy"), ImageCenterSortOrder + 1);
#else
		Channels.Add(ImageCenterChannels[0]);
		Channels.Add(ImageCenterChannels[1]);
#endif // WITH_EDITOR
	}

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

void UMovieSceneLensComponentSection::RecordFrame(FFrameNumber FrameNumber)
{
	if (const ULensComponent* const LensComponent = RecordedComponent.Get())
	{
		// Get the current distortion state of the lens component being recorded
		const FLensDistortionState CurrentState = LensComponent->GetDistortionState();
		const int32 ParameterCount = CurrentState.DistortionInfo.Parameters.Num();

		// Record the current state of each distortion parameter
		for (int32 Index = 0; Index < ParameterCount; ++Index)
		{
			DistortionParameterChannels[Index].GetData().AddKey(FrameNumber, FMovieSceneFloatValue(CurrentState.DistortionInfo.Parameters[Index]));
		}

		// Record the current state of FxFy
		FxFyChannels[0].GetData().AddKey(FrameNumber, FMovieSceneFloatValue(CurrentState.FocalLengthInfo.FxFy[0]));
		FxFyChannels[1].GetData().AddKey(FrameNumber, FMovieSceneFloatValue(CurrentState.FocalLengthInfo.FxFy[1]));

		// Record the current state of the image center
		ImageCenterChannels[0].GetData().AddKey(FrameNumber, FMovieSceneFloatValue(CurrentState.ImageCenter.PrincipalPoint[0]));
		ImageCenterChannels[1].GetData().AddKey(FrameNumber, FMovieSceneFloatValue(CurrentState.ImageCenter.PrincipalPoint[1]));
	}
}

void UMovieSceneLensComponentSection::Finalize()
{
	// Reduce keys on each of the channels that we recorded
	FKeyDataOptimizationParams Params;

	const int32 ParameterCount = DistortionParameterChannels.Num();
	for (int32 Index = 0; Index < ParameterCount; ++Index)
	{
		UE::MovieScene::Optimize(&DistortionParameterChannels[Index], Params);
	}

	UE::MovieScene::Optimize(&FxFyChannels[0], Params);
	UE::MovieScene::Optimize(&FxFyChannels[1], Params);
	UE::MovieScene::Optimize(&ImageCenterChannels[0], Params);
	UE::MovieScene::Optimize(&ImageCenterChannels[1], Params);
}

void UMovieSceneLensComponentSection::Begin(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	FScopedPreAnimatedCaptureSource CaptureSource(SharedPlaybackState, this, Params.SequenceID, EvalOptions.CompletionMode == EMovieSceneCompletionMode::RestoreState);

	FMovieSceneInstancePreAnimatedState& PreAnimatedState = SharedPlaybackState->GetPreAnimatedState();
	TArrayView<TWeakObjectPtr<>> BoundObjects = SharedPlaybackState->FindBoundObjects(Params.ObjectBindingID, Params.SequenceID);
	for (TWeakObjectPtr<> BoundObject : BoundObjects)
	{
		if (ULensComponent* LensComponent = Cast<ULensComponent>(BoundObject.Get()))
		{
			PreAnimatedState.SavePreAnimatedState(*LensComponent, FPreAnimatedLensFileTokenProducer::GetAnimTypeID(), FPreAnimatedLensFileTokenProducer());

			// Override the existing lens file settings of the lens component to use the cached, duplicate lens file
			LensComponent->SetFIZEvaluationMode(EFIZEvaluationMode::UseRecordedValues);

			// TODO: Check that the number of parameters for the lens component's current lens model matches the number of recorded parameter channels. 
		}
	}
}

void UMovieSceneLensComponentSection::Update(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	TArrayView<TWeakObjectPtr<>> BoundObjects = SharedPlaybackState->FindBoundObjects(Params.ObjectBindingID, Params.SequenceID);
	for (TWeakObjectPtr<> BoundObject : BoundObjects)
	{
		if (ULensComponent* LensComponent = Cast<ULensComponent>(BoundObject.Get()))
		{
			// Set the LensFile to use on the LensComponent, either the cached version or an override
			if (OverrideLensFile)
			{
				LensComponent->SetLensFilePicker({ false, OverrideLensFile });
			}
			else
			{
				LensComponent->SetLensFilePicker({ false, CachedLensFile });
			}

			// Nodal offset is baked in to the transform of the camera, but users can use this flag to force the LensComponent to re-evaluate
			// the LensFile and re-apply a new nodal offset instead.
			if (bReapplyNodalOffset)
			{
				LensComponent->ReapplyNodalOffset();
			}

			// Get the end frame of the current range being evaluated
			const TRange<FFrameNumber> PlaybackRange = Params.Context.GetFrameNumberRange();
			const FFrameNumber ExclusiveEndFrame = UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange);

			// Get the current distortion state of the lens component, which will be overwritten with values from the channel (if present)
			FLensDistortionState NewState = LensComponent->GetDistortionState();

			// Update the state from the distortion parameter channels
			const int32 ParameterCount = DistortionParameterChannels.Num();
			NewState.DistortionInfo.Parameters.SetNum(ParameterCount);

			for (int32 Index = 0; Index < ParameterCount; ++Index)
			{
				float NewValue = 0.0f;
				if (DistortionParameterChannels[Index].Evaluate(ExclusiveEndFrame, NewValue))
				{
					NewState.DistortionInfo.Parameters[Index] = NewValue;
				}
			}

			// Update the state from the FxFy channels
			float NewValueFx = 0.0f;
			if (FxFyChannels[0].Evaluate(ExclusiveEndFrame, NewValueFx))
			{
				NewState.FocalLengthInfo.FxFy[0] = NewValueFx;
			}

			float NewValueFy = 0.0f;
			if (FxFyChannels[1].Evaluate(ExclusiveEndFrame, NewValueFy))
			{
				NewState.FocalLengthInfo.FxFy[1] = NewValueFy;
			}

			// Update the state from the ImageCenter channels
			float NewValueCx = 0.0f;
			if (ImageCenterChannels[0].Evaluate(ExclusiveEndFrame, NewValueCx))
			{
				NewState.ImageCenter.PrincipalPoint[0] = NewValueCx;
			}

			float NewValueCy = 0.0f;
			if (ImageCenterChannels[1].Evaluate(ExclusiveEndFrame, NewValueCy))
			{
				NewState.ImageCenter.PrincipalPoint[1] = NewValueCy;
			}

			LensComponent->SetDistortionState(NewState);
		}
	}
}

void UMovieSceneLensComponentSection::End(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const
{
	SharedPlaybackState->GetPreAnimatedState().RestorePreAnimatedState();
	SharedPlaybackState->ClearObjectCaches();
}

#undef LOCTEXT_NAMESPACE
