// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/MovieSceneLensComponentTrack.h"

#include "MovieScene/MovieSceneLensComponentSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLensComponentTrack)

#define LOCTEXT_NAMESPACE "MovieSceneLensComponentTrack"

bool UMovieSceneLensComponentTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneLensComponentSection::StaticClass();
}

UMovieSceneSection* UMovieSceneLensComponentTrack::CreateNewSection()
{
	return NewObject<UMovieSceneSection>(this, UMovieSceneLensComponentSection::StaticClass(), MakeUniqueObjectName(this, UMovieSceneLensComponentSection::StaticClass(), TEXT("LensComponentSection")), RF_Transactional);
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneLensComponentTrack::GetDisplayName() const
{
	return LOCTEXT("LensComponentTrackDisplayName", "Lens Component");
}
#endif

void UMovieSceneLensComponentTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}

const TArray<UMovieSceneSection*>& UMovieSceneLensComponentTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneLensComponentTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

bool UMovieSceneLensComponentTrack::IsEmpty() const
{
	return Sections.IsEmpty();
}

void UMovieSceneLensComponentTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

void UMovieSceneLensComponentTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneLensComponentTrack::RemoveSectionAt(int32 SectionIndex)
{
	if (Sections.IsValidIndex(SectionIndex))
	{
		Sections.RemoveAt(SectionIndex);
	}
}

#undef LOCTEXT_NAMESPACE
