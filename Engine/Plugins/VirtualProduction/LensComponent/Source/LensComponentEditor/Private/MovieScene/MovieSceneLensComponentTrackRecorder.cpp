// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneLensComponentTrackRecorder.h"

#include "MovieScene/MovieSceneLensComponentTrack.h"
#include "LensComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLensComponentTrackRecorder)

#define LOCTEXT_NAMESPACE "MovieSceneLensComponentTrackRecorder"

bool FMovieSceneLensComponentTrackRecorderFactory::CanRecordObject(class UObject* InObjectToRecord) const
{
	return (InObjectToRecord->GetClass() == ULensComponent::StaticClass());
}

bool FMovieSceneLensComponentTrackRecorderFactory::CanRecordProperty(class UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const
{ 
	return false; 
}

UMovieSceneTrackRecorder* FMovieSceneLensComponentTrackRecorderFactory::CreateTrackRecorderForObject() const
{
	return NewObject<UMovieSceneLensComponentTrackRecorder>();
}

UMovieSceneTrackRecorder* FMovieSceneLensComponentTrackRecorderFactory::CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const
{
	return nullptr; 
}

FText FMovieSceneLensComponentTrackRecorderFactory::GetDisplayName() const
{ 
	return LOCTEXT("LensComponentTrackRecorderDisplayName", "Lens Component"); 
}

void UMovieSceneLensComponentTrackRecorder::CreateTrackImpl()
{
	// Find/Add a Lens Component Track to the MovieScene
	UMovieSceneLensComponentTrack* LensComponentTrack = MovieScene->FindTrack<UMovieSceneLensComponentTrack>(ObjectGuid);
	if (!LensComponentTrack)
	{
		LensComponentTrack = MovieScene->AddTrack<UMovieSceneLensComponentTrack>(ObjectGuid);
	}
	else
	{
		LensComponentTrack->RemoveAllAnimationData();
	}

	// Create a new section for the Lens Component Track and initialize it with a pointer to the component being recorded
	if (LensComponentTrack)
	{
		LensComponentSection = Cast<UMovieSceneLensComponentSection>(LensComponentTrack->CreateNewSection());

		if (UMovieSceneLensComponentSection* Section = LensComponentSection.Get())
		{
			LensComponentTrack->AddSection(*Section);

			ULensComponent* LensComponent = Cast<ULensComponent>(ObjectToRecord.Get());
			Section->Initialize(LensComponent);
		}
	}
}

void UMovieSceneLensComponentTrackRecorder::RecordSampleImpl(const FQualifiedFrameTime& CurrentTime)
{
	if (UMovieSceneLensComponentSection* Section = LensComponentSection.Get())
	{
		FFrameRate TickResolution = Section->GetTypedOuter<UMovieScene>()->GetTickResolution();
		FFrameNumber CurrentFrame = CurrentTime.ConvertTo(TickResolution).FloorToFrame();
		Section->RecordFrame(CurrentFrame);
		Section->SetEndFrame(CurrentFrame);
	}
}

void UMovieSceneLensComponentTrackRecorder::FinalizeTrackImpl()
{
	if (UMovieSceneLensComponentSection* Section = LensComponentSection.Get())
	{
		Section->Finalize();
	}
}

#undef LOCTEXT_NAMESPACE
