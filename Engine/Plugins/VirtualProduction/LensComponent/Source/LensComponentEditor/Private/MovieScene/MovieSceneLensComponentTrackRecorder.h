// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TrackRecorders/IMovieSceneTrackRecorderFactory.h"
#include "TrackRecorders/MovieSceneTrackRecorder.h"

#include "MovieScene/MovieSceneLensComponentSection.h"

#include "MovieSceneLensComponentTrackRecorder.generated.h"

/** MovieScene Track Recorder Factory for a Lens Component */
class FMovieSceneLensComponentTrackRecorderFactory : public IMovieSceneTrackRecorderFactory
{
public:
	//~ Begin IMovieSceneTrackRecorderFactory interface
	virtual bool CanRecordObject(class UObject* InObjectToRecord) const override;
	virtual bool CanRecordProperty(class UObject* InObjectToRecord, class FProperty* InPropertyToRecord) const override;

	virtual UMovieSceneTrackRecorder* CreateTrackRecorderForObject() const override;
	virtual UMovieSceneTrackRecorder* CreateTrackRecorderForProperty(UObject* InObjectToRecord, const FName& InPropertyToRecord) const override;

	virtual FText GetDisplayName() const override;
	//~ End IMovieSceneTrackRecorderFactory interface
};

/** MovieScene Track Recorder for a Lens Component */
UCLASS(BlueprintType)
class UMovieSceneLensComponentTrackRecorder : public UMovieSceneTrackRecorder
{
	GENERATED_BODY()
public:
	//~ Begin UMovieSceneTrackRecorder Interface
	virtual void CreateTrackImpl() override;
	virtual void RecordSampleImpl(const FQualifiedFrameTime& CurrentTime) override;
	virtual void FinalizeTrackImpl() override;
	virtual UMovieSceneSection* GetMovieSceneSection() const override { return LensComponentSection.Get(); }
	//~ End UMovieSceneTrackRecorder Interface

private:
	/** Section in which to record Lens Component data */
	TWeakObjectPtr<UMovieSceneLensComponentSection> LensComponentSection;
};
