// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sections/MovieSceneHookSection.h"

#include "Channels/MovieSceneFloatChannel.h"
#include "LensComponent.h"
#include "LensFile.h"

#include "MovieSceneLensComponentSection.generated.h"

#define UE_API LENSCOMPONENTEDITOR_API

struct FMovieSceneChannelProxyData;

/** Movie Scene section for Lens Component */
UCLASS(MinimalAPI)
class UMovieSceneLensComponentSection : public UMovieSceneHookSection
{
	GENERATED_BODY()

public:

	UMovieSceneLensComponentSection(const FObjectInitializer& ObjectInitializer)
		: UMovieSceneHookSection(ObjectInitializer) 
	{ 
		bRequiresRangedHook = true; 
	}

	//~ Begin UMovieSceneHookSection interface
	UE_API virtual void Begin(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;
	UE_API virtual void Update(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;
	UE_API virtual void End(TSharedRef<FSharedPlaybackState> SharedPlaybackState, const UE::MovieScene::FEvaluationHookParams& Params) const override;
	//~ End UMovieSceneHookSection interface

	/** Initialize the section with the Lens Component whose properties it will record */
	UE_API void Initialize(ULensComponent* Component);

	/** Reduce keys in each of the recorded channels */
	UE_API void Finalize();

	/** Record the current value of each distortion state property to its appropriate track */
	UE_API void RecordFrame(FFrameNumber FrameNumber);

	//~ Begin UObject interface
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostLoad() override;
	//~ End UObject interface

private:
	/** Create the channels in which distortion state values will be recorded */
	UE_API void CreateChannelProxy();

	/** Add the distortion state channels to the channel proxy */
	UE_API void UpdateChannelProxy();

#if WITH_EDITOR
	UE_API void AddChannelWithEditor(FMovieSceneChannelProxyData& ChannelProxyData, FMovieSceneFloatChannel& Channel, FText GroupName, FText ChannelName, int32 SortOrder);
#endif // WITH_EDITOR

public:
	/** If true, then every Update, the nodal offset will be re-evaluated on the lens component */
	UPROPERTY(EditAnywhere, Category = "Camera Calibration")
	bool bReapplyNodalOffset = false;

	/** LensFile asset that should be used instead of the cached LensFile during playback */
	UPROPERTY(EditAnywhere, Category = "Camera Calibration")
	TObjectPtr<ULensFile> OverrideLensFile = nullptr;

private:
	/** Saved duplicate of the LensFile asset used by the recorded Lens Component at the time of recording */
	UPROPERTY(VisibleAnywhere, Category="Camera Calibration")
	TObjectPtr<ULensFile> CachedLensFile = nullptr;

	/** Channels to store Distortion Parameter values (will be sized during initialization based on the LensComponent's LensModel) */
	UPROPERTY()
	TArray<FMovieSceneFloatChannel> DistortionParameterChannels;

	/** Channels to store FxFy values */
	UPROPERTY()
	TArray<FMovieSceneFloatChannel> FxFyChannels;

	/** Channels to store Image Center values */
	UPROPERTY()
	TArray<FMovieSceneFloatChannel> ImageCenterChannels;

	/** The Lens Model used by the recorded LensComponent */
	UPROPERTY()
	TSubclassOf<ULensModel> LensModelClass;

private:
	TWeakObjectPtr<ULensComponent> RecordedComponent;
};

#undef UE_API
