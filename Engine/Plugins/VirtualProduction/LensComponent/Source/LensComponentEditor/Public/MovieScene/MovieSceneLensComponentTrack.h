// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneNameableTrack.h"

#include "MovieSceneLensComponentTrack.generated.h"

#define UE_API LENSCOMPONENTEDITOR_API

/** Movie Scene track for Lens Component */
UCLASS(MinimalAPI)
class UMovieSceneLensComponentTrack : public UMovieSceneNameableTrack
{
	GENERATED_BODY()

public:
	//~ Begin UMovieSceneTrack interface
	UE_API virtual void AddSection(UMovieSceneSection& Section) override;
	UE_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	UE_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	UE_API virtual bool IsEmpty() const override;
	UE_API virtual void RemoveAllAnimationData() override;
	UE_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	UE_API virtual void RemoveSectionAt(int32 SectionIndex) override;

	UE_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	UE_API virtual UMovieSceneSection* CreateNewSection() override;

#if WITH_EDITORONLY_DATA
	UE_API virtual FText GetDisplayName() const override;
#endif
	//~ End UMovieSceneTrack interface

protected:
	/** Array of movie scene sections managed by this track */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};

#undef UE_API
